{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "Enhanced Input Code Quality Unreal Test Plugin",
  "Description": "Simplified testing of the Enhanced Input for Unreal Engine",
  "Category": "Testing",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "DocsURL": "",
  "MarketplaceURL": "",
  "SupportURL": "",
  "CanContainContent": false,
  "IsBetaVersion": true,
  "Installed": false,
  "Modules": [
		{
			"Name": "CQTestEnhancedInput",
			"Type": "DeveloperTool",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CQTestEnhancedInputTests",
			"Type": "DeveloperTool",
			"LoadingPhase": "Default"
		}
  ],
"Plugins": [
		{
			"Name": "EnhancedInput",
			"Enabled": true
		}
	]
}