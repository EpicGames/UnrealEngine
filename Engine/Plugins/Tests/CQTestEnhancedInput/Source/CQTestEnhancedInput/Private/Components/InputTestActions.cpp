// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/InputTestActions.h"

#include "Engine/LocalPlayer.h"
#include "EnhancedInputComponent.h"
#include "EnhancedInputSubsystems.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"

namespace {

const UInputAction* FindInputActionFromComponent(const UEnhancedInputComponent* InputComponent, const FString& InputActionName)
{
	for (const TUniquePtr<FEnhancedInputActionEventBinding>& Binding : InputComponent->GetActionEventBindings())
	{
		const UInputAction* Action = Binding->GetAction();
		if (!IsValid(Action))
		{
			continue;
		}
		if (Action->GetName().Equals(InputActionName))
		{
			return Action;
		}
	}

	return nullptr;
}

} //anonymous

void FTestAction::operator()(const APawn* Pawn)
{
	check(IsValid(Pawn));

	if (!IsValid(InputAction))
	{
		FindInputAction(Pawn);
		check(InputAction);
	}

	APlayerController* PlayerController = CastChecked<APlayerController>(Pawn->GetController());
	const ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer();
	check(IsValid(LocalPlayer));

	UEnhancedInputLocalPlayerSubsystem* EnhancedInputLocalPlayerSubsystem = LocalPlayer->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>();
	check(IsValid(EnhancedInputLocalPlayerSubsystem));

	UEnhancedPlayerInput* PlayerInput = EnhancedInputLocalPlayerSubsystem->GetPlayerInput();
	check(IsValid(PlayerInput));
	PlayerInput->InjectInputForAction(InputAction, InputActionValue);
}

void FTestAction::FindInputAction(const APawn* Pawn)
{
	if (const UEnhancedInputComponent* InputComponent = Cast<UEnhancedInputComponent>(Pawn->InputComponent))
	{
		InputAction = FindInputActionFromComponent(InputComponent, InputActionName);
	}

	if (!IsValid(InputAction))
	{
		APlayerController* PlayerController = CastChecked<APlayerController>(Pawn->GetController());
		if (const UEnhancedInputComponent* InputComponent = Cast<UEnhancedInputComponent>(PlayerController->InputComponent))
		{
			InputAction = FindInputActionFromComponent(InputComponent, InputActionName);
		}
	}
}

FInputTestActions::~FInputTestActions()
{
	StopAllActions();
}

void FInputTestActions::StopAllActions()
{
	TestActions.Empty();
	Reset();
}

void FInputTestActions::PerformAction(TFunction<void(const APawn* Pawn)> Action, TFunction<bool()> Predicate)
{
	if (Predicate)
	{
		if (!TickHandle.IsValid())
		{
			TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FInputTestActions::Tick));
		}

		TestActions.Add(FTestActionPair(MoveTemp(Action), MoveTemp(Predicate)));
	}
	else
	{
		Action(Pawn);
	}
}

void FInputTestActions::Reset()
{
	FTSTicker::RemoveTicker(TickHandle);
	TickHandle.Reset();
}

bool FInputTestActions::Tick(float DeltaTime)
{
	TestActions.RemoveAll([&](const FTestActionPair& TestActionItem) {
		TFunctionRef<bool()> Predicate = TestActionItem.Get<1>();
		return Predicate();
	});

	for (FTestActionPair& TestActionItem : TestActions)
	{
		TFunctionRef<void(const APawn* Pawn)> Action = TestActionItem.Get<0>();
		Action(Pawn);
	}

	if (TestActions.IsEmpty())
	{
		Reset();
	}

	return true;
}