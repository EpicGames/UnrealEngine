// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "InputAction.h"
#include "InputActionValue.h"

#define UE_API CQTESTENHANCEDINPUT_API

/*
//Example boiler plate

#include "CQTest.h"
#include "Components/InputTestActions.h"
#include "Components/MapTestSpawner.h"

// It is recommended that all code before the TEST_CLASS should reside in a separate file that is shared by multiple input tests
struct FInputTestAction : public FTestAction
{
	FInputTestAction(const FInputActionValue& InInputActionValue)
	{
		// Name of the InputAction assigned to the Pawn
		InputActionName = TEXT("InputActionName");

		// Value assigned to the input. Can be boolean for button presses or a FVector for axis direction
		InputActionValue = InInputActionValue;
	}
};

// Create test actions which inherits from our input action, but provides an input vector along our movement axis
struct FAxisInputAction : public FInputTestAction
{
	FAxisInputAction() : FInputTestAction(FVector2D(0.0f, 1.0f))
	{
	}
};

class FPawnTestActions : public FInputTestActions
{
public:
	explicit FPawnTestActions(APawn* Pawn) : FInputTestActions(Pawn)
	{
	}

	// Simulates player movement input actions.
	void PerformMovement()
	{
		// Perform move actions over the duration of 1 second
		PerformAction(FAxisInputAction{}, [this]() -> bool {
			if (StartTime.GetTicks() == 0)
			{
				StartTime = FDateTime::UtcNow();
			}

			FTimespan Elapsed = FDateTime::UtcNow() - StartTime;
			return Elapsed >= FTimespan::FromSeconds(1.0);
		});
	}

	FDateTime StartTime{ 0 };
};

TEST_CLASS(MyFixtureName, "InputActions.Example")
{
	TUniquePtr<FMapTestSpawner> Spawner;
	TUniquePtr<FPawnTestActions> PawnActions;
	APawn* Player;

	bool IsPlayerMoving()
	{
		return FMath::IsNearlyEqual(Player->GetVelocity().Length(), 0.0, UE_KINDA_SMALL_NUMBER);
	}

	BEFORE_EACH()
	{
		Spawner = MakeUnique<FMapTestSpawner>(TEXT("/Package/Path/To/Map"), TEXT("MapName"));
		Spawner->AddWaitUntilLoadedCommand(TestRunner);
	}

	TEST_METHOD(PlayerStartsMoving_ForDuration_EventuallyStops)
	{
		TestCommandBuilder
			.StartWhen([this]() { return nullptr != Spawner->FindFirstPlayerPawn(); })
			.Then([this]() {
				Player = Spawner->FindFirstPlayerPawn();
				PawnActions = MakeUnique<FPawnTestActions>(Player);
			})
			.Then([this]() { PawnActions->PerformMovement(); })
			.Then([this]() { ASSERT_THAT(IsTrue(IsPlayerMoving())); })
			.Until([this]() { return !IsPlayerMoving(); });
	}
};
*/

class APawn;

/** Class for testing input of a Pawn by injecting InputActions */
class FTestAction
{
public:
	virtual ~FTestAction()
	{
	}

	/**
	 * Custom input functionality to be applied on the provided Pawn.
	 *
	 * @param Pawn - Actor which the input logic will be applied to.
	 */
	UE_API virtual void operator()(const APawn* Pawn);

	FString InputActionName;
	FInputActionValue InputActionValue;

private:
	/**
	 * Finds the appropriate InputAction mapping from the Pawn using the name provided from InputActionName.
	 *
	 * @param Pawn - Pawn to search for the specified InputAction mapping.
	 */
	UE_API void FindInputAction(const APawn* Pawn);

	const UInputAction* InputAction{ nullptr };
};

/** Class for processing FTestAction objects */
class FInputTestActions
{
public:
	/**
	 * Construct the InputTestActions.
	 *
	 * @param InPawn - Pawn that will have the InputActions applied to.
	 */
	FInputTestActions(APawn* InPawn) : Pawn(InPawn)
	{
	}

	UE_API virtual ~FInputTestActions();

	/** Stops any actively running actions and clears the action queue. */
	UE_API void StopAllActions();

	/** Returns true if there are actions in the array. */
	bool HasActiveActions() const { return !TestActions.IsEmpty(); }

protected:
	/**
	 * Processes the action within the current tick.
	 *
	 * @param Action - Function with the logic to be processed on the given Pawn.
	 * @param Predicate - Function used to determine if the Action should be executed.
	 */
	UE_API void PerformAction(TFunction<void(const APawn* Pawn)> Action, TFunction<bool()> Predicate = nullptr);

	/** Clears all active timers. */
	UE_API virtual void Reset();

	APawn* Pawn{ nullptr };

private:
	/**
	 * Processes repeat actions every tick.
	 *
	 * @param DeltaTime - Time elapsed since the last tick.
	 * 
	 * @return true if tick has completed.
	 */
	UE_API bool Tick(float DeltaTime);

	FTSTicker::FDelegateHandle TickHandle;

	using FTestActionPair = TPair<TFunction<void(const APawn* Pawn)>, TFunction<bool()>>;
	TArray<FTestActionPair> TestActions;
};

#undef UE_API
