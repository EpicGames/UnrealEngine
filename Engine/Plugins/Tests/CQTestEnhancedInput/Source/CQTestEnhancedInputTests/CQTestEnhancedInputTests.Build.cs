// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CQTestEnhancedInputTests : ModuleRules
{
	public CQTestEnhancedInputTests(ReadOnlyTargetRules Target)
		: base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"EnhancedInput",
					"CQTest",
					"CQTestEnhancedInput",
				 }
			);
	}
}
