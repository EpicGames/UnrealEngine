// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTestInputTestHelper.h"

#include "Engine/World.h"
#include "InputCoreTypes.h"
#include "InputMappingContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CQTestInputTestHelper)

void UTestEnhancedInputSubsystem::Init(APlayerController* InPlayerController)
{
	check(InPlayerController);

	PlayerController = InPlayerController;
	InputComponent = CastChecked<UEnhancedInputComponent>(PlayerController->InputComponent);
	PlayerInput = CastChecked<UEnhancedPlayerInput>(PlayerController->PlayerInput);

	InitalizeUserSettings();
	SetupTestBindings();
}

void UTestEnhancedInputSubsystem::MappingListener(const FInputActionInstance& Instance)
{
	const UInputAction* SourceAction = Instance.GetSourceAction();
	check(SourceAction);

	ETriggerEvent* TriggerEvent = InputActionTriggerEventMap.Find(SourceAction->GetName());
	if (TriggerEvent)
	{
		*TriggerEvent = Instance.GetTriggerEvent();
	}
	else
	{
		InputActionTriggerEventMap.Add(SourceAction->GetName(), Instance.GetTriggerEvent());
	}
}

ETriggerEvent* UTestEnhancedInputSubsystem::GetEventForAction(const FString& InputActionName)
{
	return InputActionTriggerEventMap.Find(InputActionName);
}

void UTestEnhancedInputSubsystem::SetupTestBindings()
{
	InputMappingContext = NewObject<UInputMappingContext>(PlayerController, "TestContext");
	AddMappingContext(InputMappingContext, 0);

	BindInputAction(FCQTestInputSubsystemHelper::TestButtonActionName, EInputActionValueType::Boolean, { EKeys::A });
	BindInputAction(FCQTestInputSubsystemHelper::TestAxisActionName, EInputActionValueType::Axis1D, { EKeys::Gamepad_LeftTriggerAxis, EKeys::MouseX });

	// Generate a live mapping on the player
	FModifyContextOptions Options;
	Options.bForceImmediately = true;
	RequestRebuildControlMappings(Options);
}

void UTestEnhancedInputSubsystem::BindInputAction(const FString& InputActionName, EInputActionValueType InputActionValueType, const TArray<FKey>& Keys)
{
	UInputAction* Action = NewObject<UInputAction>(PlayerController, *InputActionName);
	Action->ValueType = InputActionValueType;

	// Bind action to the binding targets so we can check if they were called correctly, but only the first time we bind the action!
	InputComponent->BindAction(Action, ETriggerEvent::Started, this, &UTestEnhancedInputSubsystem::MappingListener);
	InputComponent->BindAction(Action, ETriggerEvent::Ongoing, this, &UTestEnhancedInputSubsystem::MappingListener);
	InputComponent->BindAction(Action, ETriggerEvent::Canceled, this, &UTestEnhancedInputSubsystem::MappingListener);
	InputComponent->BindAction(Action, ETriggerEvent::Completed, this, &UTestEnhancedInputSubsystem::MappingListener);
	InputComponent->BindAction(Action, ETriggerEvent::Triggered, this, &UTestEnhancedInputSubsystem::MappingListener);

	// Initialise input action mapping in context to be used for testing
	for (const FKey& Key : Keys)
	{
		InputMappingContext->MapKey(Action, Key);
	}
}

bool FCQTestInputSubsystemHelper::ActionExpectedEvent(const FString& InputActionName, ETriggerEvent ExpectedTriggerEvent)
{
	check(InputSubsystem);

	ETriggerEvent* TriggerEvent = InputSubsystem->GetEventForAction(InputActionName);
	if (!TriggerEvent)
	{
		return false;
	}
	
	return (*TriggerEvent) == ExpectedTriggerEvent;
}

void FCQTestInputSubsystemHelper::InitializePlayerControllerInput()
{
	check(Pawn);

	if (!Pawn->IsPlayerControlled())
	{
		UWorld* CurrentWorld = Pawn->GetWorld();
		check(CurrentWorld);

		PlayerController = NewObject<APlayerController>(CurrentWorld);
		PlayerController->Possess(Pawn);
		Pawn->PossessedBy(PlayerController);
	}
	else
	{
		PlayerController = CastChecked<APlayerController>(Pawn->GetController());
		check(PlayerController);
	}

	PlayerController->InputComponent = NewObject<UEnhancedInputComponent>(PlayerController);
	PlayerController->PlayerInput = NewObject<UEnhancedPlayerInput>(PlayerController);
	PlayerController->InitInputSystem();

	// Keep a strong reference to the newly created input system during the test execution
	InputSubsystem = TStrongObjectPtr<UTestEnhancedInputSubsystem>(NewObject<UTestEnhancedInputSubsystem>());
	InputSubsystem->Init(PlayerController);
}

void FCQTestPawnTestActions::PressButton(const FString& ButtonActionName)
{
	PerformAction(FPressButtonAction{ ButtonActionName });
}

void FCQTestPawnTestActions::HoldAxis(const FString& ExampleAxisActionName, const FInputActionValue& ActionValue, FTimespan&& Duration)
{
	PerformAction(FHoldAxisAction{ ExampleAxisActionName, ActionValue }, [this, Duration = MoveTemp(Duration)]() -> bool {
		if (StartTime.GetTicks() == 0)
		{
			StartTime = FDateTime::UtcNow();
		}

		FTimespan Elapsed = FDateTime::UtcNow() - StartTime;
		return Elapsed >= Duration;
		});
}

bool FCQTestPawnTestActions::IsTriggered(const FString& InputActionName)
{
	return InputSubsystemHelper->ActionExpectedEvent(InputActionName, ETriggerEvent::Triggered);
}

bool FCQTestPawnTestActions::IsCompleted(const FString& InputActionName)
{
	return InputSubsystemHelper->ActionExpectedEvent(InputActionName, ETriggerEvent::Completed);
}
