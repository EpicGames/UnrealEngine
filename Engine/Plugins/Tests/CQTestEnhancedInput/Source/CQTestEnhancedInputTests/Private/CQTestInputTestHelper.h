// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/InputTestActions.h"
#include "EnhancedInputComponent.h"
#include "EnhancedInputSubsystemInterface.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "UObject/StrongObjectPtr.h"

#include "CQTestInputTestHelper.generated.h"

class APawn;
class APlayerController;
class UEnhancedPlayerInput;
class UInputMappingContext;

/** Simple struct used for emulating a button interaction. Used for the InputActionTests. */
struct FPressButtonAction : public FTestAction
{
	FPressButtonAction(const FString& ButtonName)
	{
		InputActionName = ButtonName;
		InputActionValue = FInputActionValue(true);
	}
};

/** Simple struct used for emulating axis interaction. Used for the InputActionTests. */
struct FHoldAxisAction : public FTestAction
{
	FHoldAxisAction(const FString& AxisName, const FInputActionValue& ActionValue)
	{
		InputActionName = AxisName;
		InputActionValue = ActionValue;
	}
};

/** Mock input subsystems to avoid having to create an actual subsystem. */
UCLASS(NotBlueprintable, NotBlueprintType)
class UTestEnhancedInputSubsystem : public UObject, public IEnhancedInputSubsystemInterface
{
	GENERATED_BODY()

	UEnhancedPlayerInput* PlayerInput;
	UEnhancedInputUserSettings* UserSettings;

public:
	void Init(APlayerController* InPlayerController);
	UEnhancedPlayerInput* GetPlayerInput() const override { return PlayerInput; }

	// Records delegate triggering result, allowing tests to validate that they fired correctly
	void MappingListener(const FInputActionInstance& Instance);

	ETriggerEvent* GetEventForAction(const FString& InputActionName);

protected:
	virtual TMap<TObjectPtr<const UInputAction>, FInjectedInput>& GetContinuouslyInjectedInputs() override { return ContinuouslyInjectedInputs; }
	
	// Map of inputs that should be injected every frame. These inputs will be injected when ForcedInput is ticked.
	UPROPERTY(Transient)
	TMap<TObjectPtr<const UInputAction>, FInjectedInput> ContinuouslyInjectedInputs;

private:
	void SetupTestBindings();
	void BindInputAction(const FString& InputActionName, EInputActionValueType InputActionValueType, const TArray<FKey>& Keys);

	APlayerController* PlayerController;
	UEnhancedInputComponent* InputComponent;
	UInputMappingContext* InputMappingContext;
	TMap<FString, ETriggerEvent> InputActionTriggerEventMap;
};

/** Helper class used for the InputActionTests to emulate an input system as the plugin may execute tests on a Pawn with no Player or input system created. */
class FCQTestInputSubsystemHelper
{
public:
	explicit FCQTestInputSubsystemHelper(APawn* InPawn) : Pawn(InPawn)
	{
		InitializePlayerControllerInput();
	}

	bool ActionExpectedEvent(const FString& InputActionName, ETriggerEvent ExpectedTriggerEvent);

	inline static const FString TestButtonActionName = TEXT("TestButtonAction");
	inline static const FString TestAxisActionName = TEXT("TestAxisAction");

private:
	void InitializePlayerControllerInput();

	APawn* Pawn{ nullptr };
	APlayerController* PlayerController{ nullptr };

	TStrongObjectPtr<UTestEnhancedInputSubsystem> InputSubsystem{ nullptr };
};

/** Inherited InputTestAction used for testing our button and axis interactions. */
class FCQTestPawnTestActions : public FInputTestActions
{
public:
	explicit FCQTestPawnTestActions(APawn* Pawn) : FInputTestActions(Pawn)
	{
		InputSubsystemHelper = MakeUnique<FCQTestInputSubsystemHelper>(Pawn);
	}

	void PressButton(const FString& ButtonActionName);
	void HoldAxis(const FString& ExampleAxisActionName, const FInputActionValue& ActionValue, FTimespan&& Duration);
	bool IsTriggered(const FString& InputActionName);
	bool IsCompleted(const FString& InputActionName);

	FDateTime StartTime{ 0 };

	// Because we're testing the input functionality within the plugin, we need to create and handle our own input system.
	TUniquePtr<FCQTestInputSubsystemHelper> InputSubsystemHelper{ nullptr };
};