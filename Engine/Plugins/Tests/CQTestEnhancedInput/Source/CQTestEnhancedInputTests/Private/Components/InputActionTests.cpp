// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"

#include "Components/MapTestSpawner.h"
#include "Components/InputTestActions.h"
#include "CQTestInputTestHelper.h"
#include "EnhancedInputComponent.h"
#include "EnhancedInputSubsystemInterface.h"
#include "GameFramework/Pawn.h"

#if WITH_EDITOR && WITH_AUTOMATION_TESTS

TEST_CLASS(PawnActionTests, "TestFramework.CQTest.Input")
{
	APawn* TestPawn{ nullptr };
	APlayerController* PlayerController{ nullptr };
	TUniquePtr<FMapTestSpawner> Spawner{ nullptr };
	TUniquePtr<FCQTestPawnTestActions> PawnActions{ nullptr };

	BEFORE_EACH() {
		Spawner = FMapTestSpawner::CreateFromTempLevel(TestCommandBuilder);
		ASSERT_THAT(IsNotNull(Spawner));

		Spawner->AddWaitUntilLoadedCommand(TestRunner);
		TestCommandBuilder
			.StartWhen([this]() {return nullptr != Spawner->FindFirstPlayerPawn(); })
			.Then([this]() { PawnActions = MakeUnique<FCQTestPawnTestActions>(Spawner->FindFirstPlayerPawn()); });
	}

	TEST_METHOD(PawnAction_TestButtonPressAction)
	{
		TestCommandBuilder
			.Do([this]() { PawnActions->PressButton(FCQTestInputSubsystemHelper::TestButtonActionName); })
			.Then([this]() { ASSERT_THAT(IsTrue(PawnActions->IsTriggered(FCQTestInputSubsystemHelper::TestButtonActionName))); })
			.Then([this]() { ASSERT_THAT(IsTrue(PawnActions->IsCompleted(FCQTestInputSubsystemHelper::TestButtonActionName))); });
	}

	TEST_METHOD(PawnAction_TestHoldAxisAction)
	{
		TestCommandBuilder
			.Do([this]() { PawnActions->HoldAxis(FCQTestInputSubsystemHelper::TestAxisActionName, FInputActionValue(1.0f), FTimespan::FromMilliseconds(500)); })
			.Then([this]() { ASSERT_THAT(IsTrue(PawnActions->IsTriggered(FCQTestInputSubsystemHelper::TestAxisActionName))); })
			.Then([this]() { ASSERT_THAT(IsFalse(PawnActions->IsCompleted(FCQTestInputSubsystemHelper::TestAxisActionName))); })
			.Until([this]() { return PawnActions->IsCompleted(FCQTestInputSubsystemHelper::TestAxisActionName); });
	}

	TEST_METHOD(PawnAction_CanClearActiveActions)
	{
		TestCommandBuilder
			.Do([this]() { PawnActions->HoldAxis(FCQTestInputSubsystemHelper::TestAxisActionName, FInputActionValue(1.0f), FTimespan::FromSeconds(30)); })
			.Then([this]() { ASSERT_THAT(IsTrue(PawnActions->HasActiveActions())); })
			.Then([this]() { PawnActions->StopAllActions(); })
			.Then([this]() { ASSERT_THAT(IsFalse(PawnActions->HasActiveActions())); });
	}
};

#endif // WITH_EDITOR && WITH_AUTOMATION_TESTS